{
	"FileVersion": 3,
	"Version": 1,
	"VersionName": "1.0",
	"FriendlyName": "Motion Design Scene State Data Link Bridge",
	"Description": "Scene State Tasks that execute Data Link Graphs",
	"Category": "Other",
	"CreatedBy": "",
	"CreatedByURL": "",
	"DocsURL": "",
	"MarketplaceURL": "",
	"CanContainContent": true,
	"IsBetaVersion": true,
	"IsExperimentalVersion": false,
	"Installed": false,
	"Modules": [
		{
			"Name": "SceneStateDataLink",
			"Type": "Runtime",
			"LoadingPhase": "Default"
		},
		{
			"Name": "SceneStateDataLinkEditor",
			"Type": "Editor",
			"LoadingPhase": "Default"
		}
	],
	"Plugins": [
		{
			"Name": "SceneState",
			"Enabled": true
		},
		{
			"Name": "DataLink",
			"Enabled": true
		}
	]
}
