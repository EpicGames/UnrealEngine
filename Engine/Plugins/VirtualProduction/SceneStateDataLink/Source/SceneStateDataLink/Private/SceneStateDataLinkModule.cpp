// Copyright Epic Games, Inc. All Rights Reserved.

#include "Modules/ModuleManager.h"
#include "SceneStateDataLinkLog.h"

DEFINE_LOG_CATEGORY(LogSceneStateDataLink);

IMPLEMENT_MODULE(FDefaultModuleImpl, SceneStateDataLink)
