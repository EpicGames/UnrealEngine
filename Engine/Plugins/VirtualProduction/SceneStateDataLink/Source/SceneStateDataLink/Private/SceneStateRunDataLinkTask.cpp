// Copyright Epic Games, Inc. All Rights Reserved.

#include "SceneStateRunDataLinkTask.h"
#include "DataLinkExecutor.h"
#include "DataLinkExecutorArguments.h"
#include "DataLinkGraph.h"
#include "DataLinkUtils.h"
#include "Misc/ScopeExit.h"
#include "SceneStateDataLinkLog.h"
#include "SceneStateExecutionContext.h"
#include "Tasks/SceneStateTaskExecutionContext.h"

FDataLinkInstance FSceneStateDataLinkRequestTaskInstance::CreateDataLinkInstance() const
{
	return FDataLinkInstance
		{
			.DataLinkGraph = DataLinkGraph,
			.InputData = InputData
		};
}

#if WITH_EDITOR
const UScriptStruct* FSceneStateRunDataLinkTask::OnGetTaskInstanceType() const
{
	return FInstanceDataType::StaticStruct();
}

void FSceneStateRunDataLinkTask::OnBuildTaskInstance(UObject* InOuter, FStructView InTaskInstance) const
{
	FInstanceDataType& TaskInstance = InTaskInstance.Get<FInstanceDataType>();
	UE::DataLink::SetInputData(TaskInstance.DataLinkGraph, TaskInstance.InputData);
}
#endif

void FSceneStateRunDataLinkTask::OnStart(const FSceneStateExecutionContext& InContext, FStructView InTaskInstance) const
{
	FInstanceDataType& Instance = InTaskInstance.Get<FInstanceDataType>();

	const UE::SceneState::FTaskExecutionContext TaskExecutionContext(*this, InContext);

	Instance.Executor = FDataLinkExecutor::Create(FDataLinkExecutorArguments(Instance.CreateDataLinkInstance())
#if WITH_DATALINK_CONTEXT
		.SetContextName(InContext.GetExecutionContextName())
#endif
		.SetContextObject(InContext.GetContextObject())
		.SetOnOutputData(FOnDataLinkOutputData::CreateStatic(&FSceneStateRunDataLinkTask::OnOutputData, TaskExecutionContext))
		.SetOnFinished(FOnDataLinkExecutionFinished::CreateStatic(&FSceneStateRunDataLinkTask::OnFinished, TaskExecutionContext)));

	Instance.Executor->Run();
}

void FSceneStateRunDataLinkTask::OnStop(const FSceneStateExecutionContext& InContext, FStructView InTaskInstance, ESceneStateTaskStopReason InStopReason) const
{
	FInstanceDataType& Instance = InTaskInstance.Get<FInstanceDataType>();
	if (Instance.Executor.IsValid())
	{
		Instance.Executor->Stop();
		Instance.Executor.Reset();
	}
}

void FSceneStateRunDataLinkTask::OnOutputData(const FDataLinkExecutor& InExecutor, FConstStructView InOutputDataView, UE::SceneState::FTaskExecutionContext InTaskContext)
{
	if (!InOutputDataView.IsValid())
	{
		InTaskContext.FinishTask();
		return;
	}

	const FSceneStateExecutionContext* const ExecutionContext = InTaskContext.GetExecutionContext();
	if (!ExecutionContext)
	{
		InTaskContext.FinishTask();
		return;
	}

	FInstanceDataType* const Instance = InTaskContext.GetTaskInstance().GetPtr<FInstanceDataType>();

	// Ensure the instance gotten from this task context and executor remained the same
	if (!Instance || !ensure(Instance->Executor.Get() == &InExecutor))
	{
		return;
	}

	UE::SceneState::FResolvePropertyResult Result;
	if (!UE::SceneState::ResolveProperty(*ExecutionContext, Instance->OutputTarget, Result))
	{
		UE_LOGF(LogSceneStateDataLink, Warning, "[%ls] Data Link Request Task failed. Output Target could not be resolved.", *ExecutionContext->GetExecutionContextName());
		InTaskContext.FinishTask();
		return;
	}

	// The validity of these objects are guaranteed if ResolveProperty returns true
	check(Result.ResolvedReference && Result.ResolvedReference->SourceLeafProperty);

	// Validate that the reference struct and output struct match
	const FStructProperty* StructProperty = CastField<FStructProperty>(Result.ResolvedReference->SourceLeafProperty);
	if (!StructProperty)
	{
		UE_LOGF(LogSceneStateDataLink, Error, "[%ls] Data Link Request Task failed. Output Target is not bound to a struct property"
			, *ExecutionContext->GetExecutionContextName());
		InTaskContext.FinishTask();
		return;
	}

	if (!StructProperty->Struct || StructProperty->Struct != InOutputDataView.GetScriptStruct())
	{
		UE_LOGF(LogSceneStateDataLink, Warning, "[%ls] Data Link Request Task failed. Output Target struct ('%ls') and Data Link Output struct ('%ls') do not match!"
			, *ExecutionContext->GetExecutionContextName()
			, *GetNameSafe(StructProperty->Struct)
			, *GetNameSafe(InOutputDataView.GetScriptStruct()));
		InTaskContext.FinishTask();
		return;
	}

	StructProperty->Struct->CopyScriptStruct(Result.ValuePtr, InOutputDataView.GetMemory());
}

void FSceneStateRunDataLinkTask::OnFinished(const FDataLinkExecutor& InExecutor, EDataLinkExecutionResult InExecutionResult, UE::SceneState::FTaskExecutionContext InTaskContext)
{
	InTaskContext.FinishTask();
}
