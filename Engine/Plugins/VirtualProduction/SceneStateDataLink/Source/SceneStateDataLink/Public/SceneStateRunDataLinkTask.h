// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DataLinkInstance.h"
#include "SceneStatePropertyReference.h"
#include "Tasks/SceneStateTask.h"
#include "Tasks/SceneStateTaskInstance.h"
#include "SceneStateRunDataLinkTask.generated.h"

class FDataLinkExecutor;
class UDataLinkGraph;
enum class EDataLinkExecutionResult : uint8;

namespace UE::SceneState
{
	struct FTaskExecutionContext;
}

USTRUCT()
struct FSceneStateDataLinkRequestTaskInstance : public FSceneStateTaskInstance
{
	GENERATED_BODY()

	/** Creates a FDataLinkInstance from this Task Instance */
	FDataLinkInstance CreateDataLinkInstance() const;

	/** The data link graph to execute */
	UPROPERTY(EditAnywhere, Category="Data Link", meta=(NoBinding))
	TObjectPtr<UDataLinkGraph> DataLinkGraph;

	/**
	 * Input data required to run the Data Link Graph.
	 * The type of input data is determined by the Task's Data Link Graph property
	 */
	UPROPERTY(EditAnywhere, EditFixedSize, Category="Data Link", meta=(EditFixedOrder, NoBindingSelfOnly))
	TArray<FDataLinkInputData> InputData;

	/**
	 * The output struct reference to write the data link results to.
	 * NOTE: Must match the data link output struct type, or will fail to write!
	 */
	UPROPERTY(EditAnywhere, Category="Data Link", meta=(RefType="AnyStruct"))
	FSceneStatePropertyReference OutputTarget;

	/** The object handling the execution */
	TSharedPtr<FDataLinkExecutor> Executor;
};

/** Runs a Data Link Graph and writes the result to the Output Target */
USTRUCT(DisplayName="Run Data Link", Category="Data Link", meta=(UtilityTask))
struct FSceneStateRunDataLinkTask : public FSceneStateTask
{
	GENERATED_BODY()

	using FInstanceDataType = FSceneStateDataLinkRequestTaskInstance;

protected:
	//~ Begin FSceneStateTask
#if WITH_EDITOR
	virtual const UScriptStruct* OnGetTaskInstanceType() const override;
	virtual void OnBuildTaskInstance(UObject* InOuter, FStructView InTaskInstance) const override;
#endif
	virtual void OnStart(const FSceneStateExecutionContext& InContext, FStructView InTaskInstance) const override;
	virtual void OnStop(const FSceneStateExecutionContext& InContext, FStructView InTaskInstance, ESceneStateTaskStopReason InStopReason) const override;
	//~ End FSceneStateTask

private:
	/** Called when data link has broadcast output data */
	static void OnOutputData(const FDataLinkExecutor& InExecutor, FConstStructView InOutputDataView, UE::SceneState::FTaskExecutionContext InTaskContext);

	/** Called when a data link execution has finished */
	static void OnFinished(const FDataLinkExecutor& InExecutor, EDataLinkExecutionResult InExecutionResult, UE::SceneState::FTaskExecutionContext InTaskContext);
};
