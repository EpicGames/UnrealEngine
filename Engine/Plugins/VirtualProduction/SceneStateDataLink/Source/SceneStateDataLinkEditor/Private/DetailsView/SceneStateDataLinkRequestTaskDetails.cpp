// Copyright Epic Games, Inc. All Rights Reserved.

#include "SceneStateDataLinkRequestTaskDetails.h"
#include "DataLinkGraph.h"
#include "DataLinkUtils.h"
#include "IDetailChildrenBuilder.h"
#include "PropertyHandle.h"
#include "SceneStateBlueprintEditorUtils.h"
#include "SceneStateBlueprintEditorUtils.h"
#include "SceneStateRunDataLinkTask.h"

namespace UE::SceneStateDataLink
{

void FRequestTaskInstanceDetails::CustomizeHeader(TSharedRef<IPropertyHandle> InPropertyHandle, FDetailWidgetRow& InHeaderRow, IPropertyTypeCustomizationUtils& InCustomizationUtils)
{
}

FRequestTaskInstanceDetails::~FRequestTaskInstanceDetails()
{
	UDataLinkGraph::OnGraphCompiled().RemoveAll(this);
}

void FRequestTaskInstanceDetails::CustomizeChildren(TSharedRef<IPropertyHandle> InPropertyHandle, IDetailChildrenBuilder& InChildBuilder, IPropertyTypeCustomizationUtils& InCustomizationUtils)
{
	UDataLinkGraph::OnGraphCompiled().RemoveAll(this);
	UDataLinkGraph::OnGraphCompiled().AddSP(this, &FRequestTaskInstanceDetails::OnGraphCompiled);

	DataLinkGraphHandle = InPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FSceneStateDataLinkRequestTaskInstance, DataLinkGraph));
	check(DataLinkGraphHandle.IsValid());
	DataLinkGraphHandle->MarkHiddenByCustomization();
	DataLinkGraphHandle->SetOnPropertyValueChanged(FSimpleDelegate::CreateSP(this, &FRequestTaskInstanceDetails::OnGraphChanged));

	InputDataHandle = InPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FSceneStateDataLinkRequestTaskInstance, InputData));
	check(InputDataHandle.IsValid());
	InputDataHandle->MarkHiddenByCustomization();

	TSharedRef<IPropertyHandle> OutputDataHandle = InPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FSceneStateDataLinkRequestTaskInstance, OutputTarget)).ToSharedRef();
	OutputDataHandle->MarkHiddenByCustomization();

	const FGuid TaskId = SceneState::Editor::FindTaskId(InPropertyHandle);
	UE::SceneState::Editor::AssignBindingId(InputDataHandle.ToSharedRef(), TaskId);
	UE::SceneState::Editor::AssignBindingId(OutputDataHandle, TaskId);

	InChildBuilder.AddProperty(DataLinkGraphHandle.ToSharedRef());
	InChildBuilder.AddProperty(InputDataHandle.ToSharedRef());
	InChildBuilder.AddProperty(OutputDataHandle);

	UpdateInputData();
}

UDataLinkGraph* FRequestTaskInstanceDetails::GetDataLinkGraph() const
{
	UObject* DataLinkGraph = nullptr;
	if (DataLinkGraphHandle.IsValid() && DataLinkGraphHandle->GetValue(DataLinkGraph) == FPropertyAccess::Success)
	{
		return Cast<UDataLinkGraph>(DataLinkGraph);
	}
	return nullptr;
}

TArray<FDataLinkInputData>* FRequestTaskInstanceDetails::GetInputData() const
{
	void* InputData = nullptr;
	if (InputDataHandle.IsValid() && InputDataHandle->GetValueData(InputData) == FPropertyAccess::Success)
	{
		return static_cast<TArray<FDataLinkInputData>*>(InputData);
	}
	return nullptr;
}

void FRequestTaskInstanceDetails::OnGraphCompiled(UDataLinkGraph* InDataLinkGraph)
{
	if (InDataLinkGraph == GetDataLinkGraph())
	{
		OnGraphChanged();
	}
}

void FRequestTaskInstanceDetails::OnGraphChanged()
{
	if (!InputDataHandle.IsValid())
	{
		return;
	}

	InputDataHandle->NotifyPreChange();
	UpdateInputData();
	InputDataHandle->NotifyPostChange(EPropertyChangeType::ValueSet);
}

void FRequestTaskInstanceDetails::UpdateInputData()
{
	if (TArray<FDataLinkInputData>* InputData = GetInputData())
	{
		UE::DataLink::SetInputData(GetDataLinkGraph(), *InputData);
	}
}

} // UE::SceneStateDataLink
