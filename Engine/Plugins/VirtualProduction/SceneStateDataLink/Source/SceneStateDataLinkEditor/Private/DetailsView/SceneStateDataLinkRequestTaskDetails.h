// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IPropertyTypeCustomization.h"

class UDataLinkGraph;
struct FDataLinkInputData;

namespace UE::SceneStateDataLink
{

/** Details Customization for FSceneStateDataLinkRequestTaskInstance */
class FRequestTaskInstanceDetails : public IPropertyTypeCustomization
{
public:
	static TSharedRef<IPropertyTypeCustomization> MakeInstance()
	{
		return MakeShared<FRequestTaskInstanceDetails>();
	}

	virtual ~FRequestTaskInstanceDetails() override;

protected:
	//~ Begin IPropertyTypeCustomization
	virtual void CustomizeHeader(TSharedRef<IPropertyHandle> InPropertyHandle, FDetailWidgetRow& InHeaderRow, IPropertyTypeCustomizationUtils& InCustomizationUtils) override;
	virtual void CustomizeChildren(TSharedRef<IPropertyHandle> InPropertyHandle, IDetailChildrenBuilder& InChildBuilder, IPropertyTypeCustomizationUtils& InCustomizationUtils) override;
	//~ End IPropertyTypeCustomization

private:
	/** Gets the value of the data link graph property handle */
	UDataLinkGraph* GetDataLinkGraph() const;
	/** Gets the value of the input data property handle */
	TArray<FDataLinkInputData>* GetInputData() const;

	/** Called when a data link graph has compiled. Used to update the input data */
	void OnGraphCompiled(UDataLinkGraph* InDataLinkGraph);
	/** Called when the graph has changed to update the input data */
	void OnGraphChanged();

	/** Updates the input data array to match the data link graph's input pins */
	void UpdateInputData();

	TSharedPtr<IPropertyHandle> DataLinkGraphHandle;
	TSharedPtr<IPropertyHandle> InputDataHandle;
};

} // UE::SceneStateDataLink
