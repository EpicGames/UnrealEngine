// Copyright Epic Games, Inc. All Rights Reserved.

#include "SceneStateDataLinkEditorModule.h"
#include "DetailsView/SceneStateDataLinkRequestTaskDetails.h"
#include "Modules/ModuleManager.h"
#include "PropertyEditorModule.h"
#include "SceneStateRunDataLinkTask.h"

IMPLEMENT_MODULE(UE::SceneStateDataLink::FEditorModule, SceneStateDataLinkEditor)

namespace UE::SceneStateDataLink
{

void FEditorModule::StartupModule()
{
	RegisterDetailCustomizations();
}

void FEditorModule::ShutdownModule()
{
	UnregisterDetailCustomizations();
}

void FEditorModule::RegisterDetailCustomizations()
{
	FPropertyEditorModule& PropertyEditorModule = FModuleManager::LoadModuleChecked<FPropertyEditorModule>("PropertyEditor");

	PropertyEditorModule.RegisterCustomPropertyTypeLayout(CustomizedTypes.Add_GetRef(FSceneStateDataLinkRequestTaskInstance::StaticStruct()->GetFName())
		, FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FRequestTaskInstanceDetails::MakeInstance));
}

void FEditorModule::UnregisterDetailCustomizations()
{
	if (FPropertyEditorModule* PropertyEditorModule = FModuleManager::GetModulePtr<FPropertyEditorModule>("PropertyEditor"))
	{
		for (FName CustomizedType : CustomizedTypes)
		{
			PropertyEditorModule->UnregisterCustomPropertyTypeLayout(CustomizedType);
		}
		CustomizedTypes.Reset();
	}
}

} // UE::SceneStateDataLink
