// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class SceneStateDataLinkEditor : ModuleRules
{
    public SceneStateDataLinkEditor(ReadOnlyTargetRules Target) : base(Target)
    {
        PCHUsage = ModuleRules.PCHUsageMode.UseExplicitOrSharedPCHs;

        PrivateDependencyModuleNames.AddRange(
            new string[]
            {
                "Core",
                "CoreUObject",
                "DataLink",
                "Engine",
                "PropertyEditor",
                "SceneStateBlueprintEditor",
                "SceneStateDataLink",
            }
        );
    }
}
